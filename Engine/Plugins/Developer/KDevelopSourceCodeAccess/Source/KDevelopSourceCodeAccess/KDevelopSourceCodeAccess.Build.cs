// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class KDevelopSourceCodeAccess : ModuleRules
	{
		public KDevelopSourceCodeAccess(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"SourceCodeAccess",
					"DesktopPlatform",
				}
			);

			if (Target.bBuildEditor)
			{
				PrivateDependencyModuleNames.Add("HotReload");
			}

			ShortName = "KDevSCA";
		}
	}
}
