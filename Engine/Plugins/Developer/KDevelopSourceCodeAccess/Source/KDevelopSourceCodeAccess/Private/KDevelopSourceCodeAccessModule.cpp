// Copyright Epic Games, Inc. All Rights Reserved.

#include "KDevelopSourceCodeAccessModule.h"
#include "Modules/ModuleManager.h"
#include "Features/IModularFeatures.h"

IMPLEMENT_MODULE( FKDevelopSourceCodeAccessModule, KDevelopSourceCodeAccess );

void FKDevelopSourceCodeAccessModule::StartupModule()
{
	KDevelopSourceCodeAccessor.Startup();

	// Bind our source control provider to the editor
	IModularFeatures::Get().RegisterModularFeature(TEXT("SourceCodeAccessor"), &KDevelopSourceCodeAccessor );
}

void FKDevelopSourceCodeAccessModule::ShutdownModule()
{
	// unbind provider from editor
	IModularFeatures::Get().UnregisterModularFeature(TEXT("SourceCodeAccessor"), &KDevelopSourceCodeAccessor);

	KDevelopSourceCodeAccessor.Shutdown();
}

FKDevelopSourceCodeAccessor& FKDevelopSourceCodeAccessModule::GetAccessor()
{
	return KDevelopSourceCodeAccessor;
}

