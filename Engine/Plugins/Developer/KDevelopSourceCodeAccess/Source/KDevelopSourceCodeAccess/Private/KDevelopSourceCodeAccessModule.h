// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Modules/ModuleInterface.h"
#include "KDevelopSourceCodeAccessor.h"

class FKDevelopSourceCodeAccessModule : public IModuleInterface
{
public:
	/** IModuleInterface implementation */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	FKDevelopSourceCodeAccessor& GetAccessor();

private:
	FKDevelopSourceCodeAccessor KDevelopSourceCodeAccessor;
};

