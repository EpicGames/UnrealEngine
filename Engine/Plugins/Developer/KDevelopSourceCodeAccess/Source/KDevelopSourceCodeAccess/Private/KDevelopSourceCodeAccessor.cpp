// Copyright Epic Games, Inc. All Rights Reserved.

#include "KDevelopSourceCodeAccessor.h"
#include "Misc/Paths.h"
#include "Misc/UProjectInfo.h"
#include "Misc/App.h"
#include "Trace/Trace.inl"

DEFINE_LOG_CATEGORY_STATIC(LogKDevelopAccessor, Log, All);

#define LOCTEXT_NAMESPACE "KDevelopSourceCodeAccessor"

void FKDevelopSourceCodeAccessor::Startup()
{
	// Cache this so we don't have to do it on a background thread
	GetSolutionPath();
	
	// FIXME: look for kdevelop and cache the path
	
}

void FKDevelopSourceCodeAccessor::Shutdown()
{
}

bool FKDevelopSourceCodeAccessor::OpenSolution()
{
	FString SolutionPath = GetSolutionPath();
	return OpenSolutionAtPath(SolutionPath);
}

bool FKDevelopSourceCodeAccessor::OpenSolutionAtPath(const FString& InSolutionPath)
{
	if (IsIDERunning())
	{
		// use qdbus to open the project within session?
		STUBBED("OpenSolution: KDevelop is running, use qdbus to open the project within session?");
		return false;
	}

	FString SolutionPath = InSolutionPath;
	if (!SolutionPath.EndsWith(TEXT(".kdev4")))
	{
		SolutionPath += TEXT(".kdev4");
	}

	FString IDEPath;
	if (!CanRunKDevelop(IDEPath))
	{
		UE_LOGF(LogKDevelopAccessor, Warning, "FKDevelopSourceCodeAccessor::OpenSourceFiles: cannot find kdevelop binary");
		return false;
	}
	
	FProcHandle Proc = FPlatformProcess::CreateProc(*IDEPath, *SolutionPath, true, false, false, nullptr, 0, nullptr, nullptr);
	if (Proc.IsValid())
	{
		FPlatformProcess::CloseProc(Proc);
		return true;
	}
	return false;
}

bool FKDevelopSourceCodeAccessor::DoesSolutionExist() const
{
	FString SolutionPath = GetSolutionPath();
	UE_LOGF(LogKDevelopAccessor, Display, "SolutionPath: %ls", *SolutionPath);
	return FPaths::FileExists(SolutionPath);
}

bool FKDevelopSourceCodeAccessor::CanRunKDevelop(FString& OutPath) const
{
	// FIXME: search properly
	OutPath = TEXT("/usr/bin/kdevelop");	
	return FPaths::FileExists(OutPath);
}

bool FKDevelopSourceCodeAccessor::OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths)
{
	if (IsIDERunning())
	{
		// use qdbus
		STUBBED("OpenSourceFiles: KDevelop is running, use qdbus");
		return false;
	}
	
	STUBBED("FKDevelopSourceCodeAccessor::OpenSourceFiles");
	return false;
}

bool FKDevelopSourceCodeAccessor::AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules)
{
	return false;
}

bool FKDevelopSourceCodeAccessor::OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber)
{
	// column & line numbers are 1-based, so dont allow zero
	if(LineNumber == 0)
	{
		LineNumber++;
	}
	if(ColumnNumber == 0)
{
		ColumnNumber++;
	}

	// Automatically fail if there's already an attempt in progress
	if (IsIDERunning())
	{
		// use qdbus
		STUBBED("OpenFileAtLine: KDevelop is running, use qdbus");
		return false;
	}

	STUBBED("FKDevelopSourceCodeAccessor::OpenFileAtLine");
	return false;
}

bool FKDevelopSourceCodeAccessor::SaveAllOpenDocuments() const
{
	STUBBED("FKDevelopSourceCodeAccessor::SaveAllOpenDocuments");
	return false;
}

void FKDevelopSourceCodeAccessor::Tick(const float DeltaTime)
{
}

bool FKDevelopSourceCodeAccessor::CanAccessSourceCode() const
{
	FString Path;
	return CanRunKDevelop(Path);
}

FName FKDevelopSourceCodeAccessor::GetFName() const
{
	return FName("KDevelop");
}

FText FKDevelopSourceCodeAccessor::GetNameText() const
{
	return LOCTEXT("KDevelopDisplayName", "KDevelop 4.x");
}

FText FKDevelopSourceCodeAccessor::GetDescriptionText() const
{
	return LOCTEXT("KDevelopDisplayDesc", "Open source code files in KDevelop 4.x");
}

bool FKDevelopSourceCodeAccessor::IsIDERunning()
{
	// FIXME: implement
	STUBBED("FKDevelopSourceCodeAccessor::IsIDERunning");
	return false;
}

FString FKDevelopSourceCodeAccessor::GetSolutionPath() const
{
	if(IsInGameThread())
	{
		CachedSolutionPath = FPaths::ProjectDir();
		
		if (!FUProjectDictionary::GetDefault().IsForeignProject(CachedSolutionPath))
		{
			CachedSolutionPath = FPaths::Combine(FPaths::RootDir(), TEXT("UnrealEditor.kdev4"));
		}
		else
		{
			FString BaseName = FApp::HasProjectName() ? FApp::GetProjectName() : FPaths::GetBaseFilename(CachedSolutionPath);
			CachedSolutionPath = FPaths::Combine(CachedSolutionPath, BaseName + TEXT(".kdev4"));
		}
	}
	return CachedSolutionPath;
}

#undef LOCTEXT_NAMESPACE
