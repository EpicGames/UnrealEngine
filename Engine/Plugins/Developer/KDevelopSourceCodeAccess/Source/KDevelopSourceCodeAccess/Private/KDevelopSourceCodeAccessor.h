// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "ISourceCodeAccessor.h"

class FKDevelopSourceCodeAccessor : public ISourceCodeAccessor
{
public:
	/** Initialise internal systems, register delegates etc. */
	void Startup();

	/** Shut down internal systems, unregister delegates etc. */
	void Shutdown();

	/** ISourceCodeAccessor implementation */
	virtual void RefreshAvailability() override { }
	virtual bool CanAccessSourceCode() const override;
	virtual FName GetFName() const override;
	virtual FText GetNameText() const override;
	virtual FText GetDescriptionText() const override;
	virtual bool OpenSolution() override;
	virtual bool OpenSolutionAtPath(const FString& InSolutionPath) override;
	virtual bool DoesSolutionExist() const override;
	virtual bool OpenFileAtLine(const FString& FullPath, int32 LineNumber, int32 ColumnNumber = 0) override;
	virtual bool OpenSourceFiles(const TArray<FString>& AbsoluteSourcePaths) override;
	virtual bool AddSourceFiles(const TArray<FString>& AbsoluteSourcePaths, const TArray<FString>& AvailableModules) override;
	virtual bool SaveAllOpenDocuments() const override;
	virtual void Tick(const float DeltaTime) override;

private:

	/** String storing the solution path obtained from the module manager to avoid having to use it on a thread */
	mutable FString CachedSolutionPath;

	/** Tests if KDevelop is present and returns path to it */
	bool CanRunKDevelop(FString& OutPath) const;

	/** Check if KDevelop is already running */
	bool IsIDERunning();

	/** Gets solution path */
	FString GetSolutionPath() const;
};
